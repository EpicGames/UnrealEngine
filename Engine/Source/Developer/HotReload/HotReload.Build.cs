// Copyright Epic Games, Inc. All Rights Reserved.
using UnrealBuildTool;

public class HotReload : ModuleRules
{
	public HotReload(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Analytics",
				"DirectoryWatcher",
				"DesktopPlatform",
				"Projects"
			}
		);

        if (Target.bCompileAgainstEngine)
        {
            PrivateDependencyModuleNames.AddRange(
				new string[] 
				{
					"Engine",
					"UnrealEd", 
				}
			);
        }

		if(Target.bWithLiveCoding)
		{
			PrivateIncludePathModuleNames.Add("LiveCoding");
		}
	}
}
