// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "HAL/PlatformProcess.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/FileManager.h"
#include "Misc/CoreMisc.h"
#include "Misc/Paths.h"
#include "Misc/QueuedThreadPool.h"
#include "Misc/OutputDeviceNull.h"
#include "Stats/Stats.h"
#include "Async/AsyncWork.h"
#include "Containers/Ticker.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FeedbackContext.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "Serialization/ArchiveUObject.h"
#include "UObject/GarbageCollection.h"
#include "UObject/Class.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UnrealType.h"
#include "Misc/PackageName.h"
#include "IHotReload.h"
#include "IDirectoryWatcher.h"
#include "DirectoryWatcherModule.h"
#include "HotReloadLog.h"
#include "AnalyticsEventAttribute.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "Interfaces/IPluginManager.h"
#include "DesktopPlatformModule.h"
#include "HAL/LowLevelMemTracker.h"
#if WITH_ENGINE
#include "Engine/Engine.h"
#include "EngineAnalytics.h"
#endif
#if WITH_EDITOR
#include "Kismet2/ReloadUtilities.h"
#endif
#include "Misc/ScopeExit.h"
#include "Algo/Transform.h"

#if WITH_LIVE_CODING
#include "ILiveCodingModule.h"
#endif

#if WITH_EDITOR
#include "Editor.h"
#endif

#include "Containers/Ticker.h"

DEFINE_LOG_CATEGORY(LogHotReload);

#define LOCTEXT_NAMESPACE "HotReload"

LLM_DEFINE_TAG(HotReload);

namespace EThreeStateBool
{
	enum Type
	{
		False,
		True,
		Unknown
	};

	static bool ToBool(EThreeStateBool::Type Value)
	{
		switch (Value)
		{
		case EThreeStateBool::False:
			return false;
		case EThreeStateBool::True:
			return true;
		default:
			UE_LOGF(LogHotReload, Fatal, "Can't convert EThreeStateBool to bool value because it's Unknown");			
			break;
		}
		return false;
	}

	static EThreeStateBool::Type FromBool(bool Value)
	{
		return Value ? EThreeStateBool::True : EThreeStateBool::False;
	}
};

#if WITH_HOT_RELOAD
class FScopedHotReload
{
public:
	FScopedHotReload(TUniquePtr<FReload>& InUniquePtr, const TArray<UPackage*>& InPackages)
		: UniquePtr(InUniquePtr)
	{
		UniquePtr.Reset(new FReload(EActiveReloadType::HotReload, TEXT("HOTRELOAD"), InPackages, *GLog));
	}

	FScopedHotReload(TUniquePtr<FReload>& InUniquePtr)
		: UniquePtr(InUniquePtr)
	{
		UniquePtr.Reset(new FReload(EActiveReloadType::HotReload, TEXT("HOTRELOAD"), *GLog));
	}

	~FScopedHotReload()
	{
		UniquePtr.Reset();
	}

private:
	TUniquePtr<FReload>& UniquePtr;
};
#endif // WITH_HOT_RELOAD

/**
 * Module for HotReload support
 */
class FHotReloadModule : public IHotReloadModule, FSelfRegisteringExec
{
public:

	FHotReloadModule()
	{
		ModuleCompileReadPipe = nullptr;
		bRequestCancelCompilation = false;
		bIsAnyGameModuleLoaded = EThreeStateBool::Unknown;
		bDirectoryWatcherInitialized = false;
	}

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	/** IHotReloadInterface implementation */
	virtual void SaveConfig() override;
	virtual bool RecompileModule(const FName InModuleName, FOutputDevice &Ar, ERecompileModuleFlags Flags) override;
	virtual bool IsCurrentlyCompiling() const override { return ModuleCompileProcessHandle.IsValid(); }
	virtual void RequestStopCompilation() override { bRequestCancelCompilation = true; }
	virtual ECompilationResult::Type RebindPackages(const TArray<UPackage*>& Packages, EHotReloadFlags Flags, FOutputDevice &Ar) override;
	virtual ECompilationResult::Type DoHotReloadFromEditor(EHotReloadFlags Flags) override;
	virtual FModuleCompilerStartedEvent& OnModuleCompilerStarted() override { return ModuleCompilerStartedEvent; }
	virtual FModuleCompilerFinishedEvent& OnModuleCompilerFinished() override { return ModuleCompilerFinishedEvent; }
	virtual FString GetModuleCompileMethod(FName InModuleName) override;
	virtual bool IsAnyGameModuleLoaded() override;

protected:
	/** FSelfRegisteringExec implementation */
	virtual bool Exec_Dev( UWorld* Inworld, const TCHAR* Cmd, FOutputDevice& Ar ) override;

private:
	/**
	 * Enumerates compilation methods for modules.
	 */
	enum class EModuleCompileMethod
	{
		Runtime,
		External,
		Unknown
	};

	/**
	 * Helper structure to hold on to module state while asynchronously recompiling DLLs
	 */
	struct FModuleToRecompile
	{
		/** Name of the module */
		FName ModuleName;

		/** Desired module file name suffix, or empty string if not needed */
		FString ModuleFileSuffix;

		/** The module file name to use after a compilation succeeds, or an empty string if not changing */
		FString NewModuleFilename;
	};

	/**
	 * Helper structure to store the compile time and method for a module
	 */
	struct FModuleCompilationData
	{
		/** Has a timestamp been set for the .dll file */
		bool bHasFileTimeStamp;

		/** Last known timestamp for the .dll file */
		FDateTime FileTimeStamp;

		/** Last known compilation method of the .dll file */
		EModuleCompileMethod CompileMethod;

		FModuleCompilationData()
			: bHasFileTimeStamp(false)
			, CompileMethod(EModuleCompileMethod::Unknown)
		{ }
	};

	/**
	 * Adds a callback to directory watcher for the game binaries folder.
	 */
	void RefreshHotReloadWatcher();

	/**
	 * Adds a directory watch on the binaries directory under the given folder.
	 */
	void AddHotReloadDirectory(IDirectoryWatcher* DirectoryWatcher, const FString& BaseDir);

	/**
	 * Removes a directory watcher callback
	 */
	void ShutdownHotReloadWatcher();

	/**
	 * Performs hot-reload from IDE (when game DLLs change)
	 */
	void DoHotReloadFromIDE(const TMap<FName, FString>& NewModules);

	/**
	* Performs internal module recompilation
	*/
	ECompilationResult::Type RebindPackagesInternal(const TArray<UPackage*>& Packages, const TArray<FName>& DependentModules, EHotReloadFlags Flags, FOutputDevice& Ar);

	/**
	 * Does the actual hot-reload, unloads old modules, loads new ones
	 */
	ECompilationResult::Type DoHotReloadInternal(const TMap<FName, FString>& ChangedModuleNames, const TArray<UPackage*>& Packages, const TArray<FName>& InDependentModules, FOutputDevice& HotReloadAr);

#if WITH_ENGINE
	void RegisterForReinstancing(UClass* OldClass, UClass* NewClass, EHotReloadedClassFlags Flags);
	void ReinstanceClasses();
#endif

	/**
	 * Tick function for FTSTicker: checks for re-loaded modules and does hot-reload from IDE
	 */
	bool Tick(float DeltaTime);

	/**
	 * Directory watcher callback
	 */
	void OnHotReloadBinariesChanged(const TArray<struct FFileChangeData>& FileChanges);

	/**
	 * Strips hot-reload suffix from module filename.
	 */
	static void StripModuleSuffixFromFilename(FString& InOutModuleFilename, const FString& ModuleName);

	/**
	 * Sends analytics event about the re-load
	 */
	static void RecordAnalyticsEvent(const TCHAR* ReloadFrom, ECompilationResult::Type Result, double Duration, int32 PackageCount, int32 DependentModulesCount);

	/**
	 * Declares a function type that is executed after a module recompile has finished.
	 *
	 * ChangedModules: A map between the names of the modules that have changed and their filenames.
	 * bRecompileFinished: Signals whether compilation has finished.
	 * CompilationResult: Shows whether compilation was successful or not.
	 */
	typedef TFunction<void(const TMap<FName, FString>& ChangedModules, bool bRecompileFinished, ECompilationResult::Type CompilationResult)> FRecompileModulesCallback;

	/** Called for successfully re-complied module */
	void OnModuleCompileSucceeded(FName ModuleName, const FString& ModuleFilename);

	/** Returns arguments to pass to UnrealBuildTool when compiling modules */
	static FString MakeUBTArgumentsForModuleCompiling();

#if WITH_HOT_RELOAD
	/** 
	 *	Starts compiling DLL files for one or more modules.
	 *
	 *	@param ModuleNames The list of modules to compile.
	 *	@param InRecompileModulesCallback Callback function to make when module recompiles.
	 *	@param Ar
	 *	@param InAdditionalCmdLineArgs Additional arguments to pass to UBT.
	 *  @param Flags Compilation flags
	 *	@return true if successful, false otherwise.
	 */
	bool StartCompilingModuleDLLs(const TArray< FModuleToRecompile >& ModuleNames,
		FRecompileModulesCallback&& InRecompileModulesCallback, FOutputDevice& Ar,
		const FString& InAdditionalCmdLineArgs, ERecompileModuleFlags Flags);
#endif

	/** Launches UnrealBuildTool with the specified command line parameters */
	bool InvokeUnrealBuildToolForCompile(const FString& InCmdLineParams, FOutputDevice &Ar);

	/** Checks to see if a pending compilation action has completed and optionally waits for it to finish.  If completed, fires any appropriate callbacks and reports status provided bFireEvents is true. */
	void CheckForFinishedModuleDLLCompile(EHotReloadFlags Flags, bool& bCompileStillInProgress, bool& bCompileSucceeded, FOutputDevice& Ar, bool bFireEvents = true);

	/** Called when the compile data for a module need to be update in memory and written to config */
	void UpdateModuleCompileData(FName ModuleName);

	/** Called when a new module is added to the manager to get the saved compile data from config */
	static void ReadModuleCompilationInfoFromConfig(FName ModuleName, FModuleCompilationData& CompileData);

	/** Saves the module's compile data to config */
	static void WriteModuleCompilationInfoToConfig(FName ModuleName, const FModuleCompilationData& CompileData);

	/** Access the module's file and read the timestamp from the file system. Returns true if the timestamp was read successfully. */
	bool GetModuleFileTimeStamp(FName ModuleName, FDateTime& OutFileTimeStamp) const;

	/** Checks if the specified array of modules to recompile contains only game modules */
	bool ContainsOnlyGameModules(const TArray< FModuleToRecompile >& ModuleNames) const;

	/** Callback registered with ModuleManager to know if any new modules have been loaded */
	void ModulesChangedCallback(FName ModuleName, EModuleChangeReason ReasonForChange);

	/** Callback registered with PluginManager to know if any new plugins have been created */
	void PluginMountedCallback(IPlugin& Plugin);

	/** FTSTicker delegate (hot-reload from IDE) */
	FTickerDelegate TickerDelegate;

	/** Handle to the registered TickerDelegate */
	FTSTicker::FDelegateHandle TickerDelegateHandle;

	/** Handle to the registered delegate above */
	TMap<FString, FDelegateHandle> BinariesFolderChangedDelegateHandles;

	/** True if currently hot-reloading from editor (suppresses hot-reload from IDE) */
	bool bIsHotReloadingFromEditor;
	
	/** New module DLLs detected by the directory watcher */
	TMap<FName, FString> DetectedNewModules;

	/** Modules that have been recently recompiled from the editor **/
	TSet<FName> ModulesRecentlyCompiledInTheEditor;

	/** Array of modules that we're currently recompiling */
	TArray< FName > ModulesBeingCompiled;

	/** Array of modules that we're going to recompile */
	TArray< FModuleToRecompile > ModulesThatWereBeingRecompiled;

	/** Last known compilation data for each module */
	TMap<FName, TSharedRef<FModuleCompilationData>> ModuleCompileData;

	/** Multicast delegate which will broadcast a notification when the compiler starts */
	FModuleCompilerStartedEvent ModuleCompilerStartedEvent;
	
	/** Multicast delegate which will broadcast a notification when the compiler finishes */
	FModuleCompilerFinishedEvent ModuleCompilerFinishedEvent;

	/** When compiling a module using an external application, stores the handle to the process that is running */
	FProcHandle ModuleCompileProcessHandle;

	/** When compiling a module using an external application, this is the process read pipe handle */
	void* ModuleCompileReadPipe;

	/** When compiling a module using an external application, this is the text that was read from the read pipe handle */
	FString ModuleCompileReadPipeText;

	/** Callback to execute after an asynchronous recompile has completed (whether successful or not.) */
	FRecompileModulesCallback RecompileModulesCallback;

	/** true if we should attempt to cancel the current async compilation */
	bool bRequestCancelCompilation;

	/** Tracks the validity of the game module existence */
	EThreeStateBool::Type bIsAnyGameModuleLoaded;

	/** True if the directory watcher has been successfully initialized */
	bool bDirectoryWatcherInitialized;

	/** Keeps record of hot-reload session starting time. */
	double HotReloadStartTime;

	/** Current reload object */
	TUniquePtr<FReload> Reload;
};

IMPLEMENT_MODULE(FHotReloadModule, HotReload);

namespace HotReloadDefs
{
	static const TCHAR* CompilationInfoConfigSection = TEXT("ModuleFileTracking");

	// These strings should match the values of the enum EModuleCompileMethod in ModuleManager.h
	// and should be handled in ReadModuleCompilationInfoFromConfig() & WriteModuleCompilationInfoToConfig() below
	static const TCHAR* CompileMethodRuntime = TEXT("Runtime");
	static const TCHAR* CompileMethodExternal = TEXT("External");
	static const TCHAR* CompileMethodUnknown = TEXT("Unknown");

	// Add one minute epsilon to timestamp comparision
	const static FTimespan TimeStampEpsilon(0, 1, 0);
}

namespace UEHotReload_Private
{
	/**
	 * Gets editor runs directory.
	 */
	FString GetEditorRunsDir()
	{
		FString TempDir = FPaths::EngineIntermediateDir();

		return FPaths::Combine(*TempDir, TEXT("EditorRuns"));
	}

	/**
	 * Creates a file that informs UBT that the editor is currently running.
	 */
	void CreateFileThatIndicatesEditorRunIfNeeded()
	{
#if WITH_EDITOR
		IPlatformFile& FS = IPlatformFile::GetPlatformPhysical();

		FString EditorRunsDir = GetEditorRunsDir();
		FString FileName = FPaths::Combine(*EditorRunsDir, *FString::Printf(TEXT("%d"), FPlatformProcess::GetCurrentProcessId()));

		if (FS.FileExists(*FileName))
		{
			if (!GIsEditor)
			{
				FS.DeleteFile(*FileName);
			}
		}
		else
		{
			if (GIsEditor)
			{
				if (!FS.CreateDirectory(*EditorRunsDir))
				{
					return;
				}

				delete FS.OpenWrite(*FileName); // Touch file.
			}
		}
#endif // WITH_EDITOR
	}

	/**
	 * Deletes file left by CreateFileThatIndicatesEditorRunIfNeeded function.
	 */
	void DeleteFileThatIndicatesEditorRunIfNeeded()
	{
#if WITH_EDITOR
		IPlatformFile& FS = IPlatformFile::GetPlatformPhysical();

		FString EditorRunsDir = GetEditorRunsDir();
		FString FileName = FPaths::Combine(*EditorRunsDir, *FString::Printf(TEXT("%d"), FPlatformProcess::GetCurrentProcessId()));

		if (FS.FileExists(*FileName))
		{
			FS.DeleteFile(*FileName);
		}
#endif // WITH_EDITOR
	}

	/**
	 * Gets all currently loaded game module names and optionally, the file names for those modules
	 */
	TArray<FString> GetGameModuleNames(const FModuleManager& ModuleManager)
	{
		TArray<FString> Result;

		// Ask the module manager for a list of currently-loaded gameplay modules
		TArray<FModuleStatus> ModuleStatuses;
		ModuleManager.QueryModules(ModuleStatuses);

		for (FModuleStatus& ModuleStatus : ModuleStatuses)
		{
			// We only care about game modules that are currently loaded
			if (ModuleStatus.bIsLoaded && ModuleStatus.bIsGameModule)
			{
				Result.Add(MoveTemp(ModuleStatus.Name));
			}
		}

		return Result;
	}

	/**
	 * Gets all currently loaded game module names and optionally, the file names for those modules
	 */
	TMap<FName, FString> GetGameModuleFilenames(const FModuleManager& ModuleManager)
	{
		TMap<FName, FString> Result;

		// Ask the module manager for a list of currently-loaded gameplay modules
		TArray< FModuleStatus > ModuleStatuses;
		ModuleManager.QueryModules(ModuleStatuses);

		for (FModuleStatus& ModuleStatus : ModuleStatuses)
		{
			// We only care about game modules that are currently loaded
			if (ModuleStatus.bIsLoaded && ModuleStatus.bIsGameModule)
			{
				Result.Add(*ModuleStatus.Name, MoveTemp(ModuleStatus.FilePath));
			}
		}

		return Result;
	}

	struct FPackagesAndDependentNames
	{
		TArray<UPackage*> Packages;
		TArray<FName> DependentNames;
	};

	/**
	 * Gets named packages and the names dependents.
	 */
	FPackagesAndDependentNames SplitByPackagesAndDependentNames(const TArray<FString>& ModuleNames)
	{
		FPackagesAndDependentNames Result;

		for (const FString& ModuleName : ModuleNames)
		{
			FString PackagePath = TEXT("/Script/") + ModuleName;

			if (UPackage* Package = FindPackage(nullptr, *PackagePath))
			{
				Result.Packages.Add(Package);
			}
			else
			{
				Result.DependentNames.Add(*ModuleName);
			}
		}

		return Result;
	}
}

namespace HotReloadModule
{
	inline bool DisableHotReloadForThisRun()
	{
		// HotReload will be disabled when either running in -unattended mode or running a commandlet.
		return FApp::IsUnattended() || IsRunningCommandlet();
	}
}

void FHotReloadModule::StartupModule()
{
	LLM_SCOPE_BYTAG(HotReload);

	if (HotReloadModule::DisableHotReloadForThisRun())
	{
		return;
	}

	UEHotReload_Private::CreateFileThatIndicatesEditorRunIfNeeded();

	bIsHotReloadingFromEditor = false;

#if WITH_ENGINE
	// Register re-instancing delegate (Core)
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FCoreUObjectDelegates::RegisterClassForHotReloadReinstancingDelegate.AddRaw(this, &FHotReloadModule::RegisterForReinstancing);
	FCoreUObjectDelegates::ReinstanceHotReloadedClassesDelegate.AddRaw(this, &FHotReloadModule::ReinstanceClasses);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

	// Register directory watcher delegate
	RefreshHotReloadWatcher();

	// Register hot-reload from IDE ticker
	TickerDelegate = FTickerDelegate::CreateRaw(this, &FHotReloadModule::Tick);
	TickerDelegateHandle = FTSTicker::GetCoreTicker().AddTicker(TickerDelegate);

	FModuleManager::Get().OnModulesChanged().AddRaw(this, &FHotReloadModule::ModulesChangedCallback);

	IPluginManager::Get().OnNewPluginCreated().AddRaw(this, &FHotReloadModule::PluginMountedCallback);
}

void FHotReloadModule::ShutdownModule()
{
	if (HotReloadModule::DisableHotReloadForThisRun())
	{
		return;
	}

	FTSTicker::RemoveTicker(TickerDelegateHandle);
	ShutdownHotReloadWatcher();

	UEHotReload_Private::DeleteFileThatIndicatesEditorRunIfNeeded();
}

bool FHotReloadModule::Exec_Dev( UWorld* Inworld, const TCHAR* Cmd, FOutputDevice& Ar )
{
#if !UE_BUILD_SHIPPING
	if ( FParse::Command( &Cmd, TEXT( "Module" ) ) )
	{
#if WITH_HOT_RELOAD
		// Recompile <ModuleName>
		if( FParse::Command( &Cmd, TEXT( "Recompile" ) ) )
		{
			const FString ModuleNameStr = FParse::Token( Cmd, 0 );
			if( !ModuleNameStr.IsEmpty() )
			{
				const FName ModuleName( *ModuleNameStr );
				RecompileModule(ModuleName, Ar, ERecompileModuleFlags::ReloadAfterRecompile | ERecompileModuleFlags::FailIfGeneratedCodeChanges);
			}

			return true;
		}
#endif // WITH_HOT_RELOAD
	}
#endif // !UE_BUILD_SHIPPING
	return false;
}

void FHotReloadModule::SaveConfig()
{
	// Find all the modules
	TArray<FModuleStatus> Modules;
	FModuleManager::Get().QueryModules(Modules);

	// Update the compile data for each one
	for( const FModuleStatus &Module : Modules )
	{
		UpdateModuleCompileData(*Module.Name);
	}
}

FString FHotReloadModule::GetModuleCompileMethod(FName InModuleName)
{
	LLM_SCOPE_BYTAG(HotReload);

	if (!ModuleCompileData.Contains(InModuleName))
	{
		UpdateModuleCompileData(InModuleName);
	}

	switch(ModuleCompileData.FindChecked(InModuleName).Get().CompileMethod)
	{
	case EModuleCompileMethod::External:
		return HotReloadDefs::CompileMethodExternal;
	case EModuleCompileMethod::Runtime:
		return HotReloadDefs::CompileMethodRuntime;
	default:
		return HotReloadDefs::CompileMethodUnknown;
	}
}

bool FHotReloadModule::RecompileModule(const FName InModuleName, FOutputDevice &Ar, ERecompileModuleFlags Flags)
{
#if WITH_HOT_RELOAD

#if WITH_LIVE_CODING
	ILiveCodingModule* LiveCoding = FModuleManager::GetModulePtr<ILiveCodingModule>(LIVE_CODING_MODULE_NAME);
	if (LiveCoding != nullptr && LiveCoding->IsEnabledForSession())
	{
		UE_LOGF(LogHotReload, Error, "Unable to hot-reload modules while Live Coding is enabled.");
		return false;
	}
#endif

	UE_LOGF(LogHotReload, Log, "Recompiling module %ls...", *InModuleName.ToString());

	// This is an internal request for hot-reload (not from IDE)
	TGuardValue<bool> GuardHotReloadingFromEditorFlag(bIsHotReloadingFromEditor, true);

	// A list of modules that have been recompiled in the editor is going to prevent false
	// hot-reload from IDE events as this call is blocking any potential callbacks coming from the filesystem
	// and bIsHotReloadingFromEditor may not be enough to prevent those from being treated as actual hot-reload from IDE modules
	ModulesRecentlyCompiledInTheEditor.Empty();


	FFormatNamedArguments Args;
	Args.Add( TEXT("CodeModuleName"), FText::FromName( InModuleName ) );
	const FText StatusUpdate = FText::Format( NSLOCTEXT("ModuleManager", "Recompile_SlowTaskName", "Compiling {CodeModuleName}..."), Args );

	FScopedSlowTask SlowTask(2, StatusUpdate);
	SlowTask.MakeDialog();

	ModuleCompilerStartedEvent.Broadcast(false); // we never perform an async compile

	FModuleManager& ModuleManager = FModuleManager::Get();

	// Update our set of known modules, in case we don't already know about this module
	ModuleManager.AddModule( InModuleName );

	// Only use rolling module names if the module was already loaded into memory.  This allows us to try compiling
	// the module without actually having to unload it first.
	const bool bWasModuleLoaded = ModuleManager.IsModuleLoaded( InModuleName );

	SlowTask.EnterProgressFrame();

	/**
	 * Tries to recompile the specified DLL using UBT. Does not interact with modules. This is a low level routine.
	 *
	 * @param ModuleNames List of modules to recompile, including the module name and optional file suffix.
	 * @param Ar Output device for logging compilation status.
	 * @param bForceCodeProject Even if it's a non-code project, treat it as code-based project
	 */
	auto RecompileModuleDLLs = [this, &Ar, Flags](const TArray< FModuleToRecompile >& ModuleNames)
	{
		bool bCompileSucceeded = false;
		const FString AdditionalArguments = MakeUBTArgumentsForModuleCompiling();
		if (StartCompilingModuleDLLs(ModuleNames, nullptr, Ar, AdditionalArguments, Flags))
		{
			bool bCompileStillInProgress = false;
			CheckForFinishedModuleDLLCompile( EHotReloadFlags::WaitForCompletion, bCompileStillInProgress, bCompileSucceeded, Ar );
		}
		return bCompileSucceeded;
	};

	// First, try to compile the module.  If the module is already loaded, we won't unload it quite yet.  Instead
	// make sure that it compiles successfully.

	// Find a unique file name for the module
	FModuleToRecompile ModuleToRecompile;
	ModuleToRecompile.ModuleName = InModuleName;
	ModuleManager.MakeUniqueModuleFilename( InModuleName, ModuleToRecompile.ModuleFileSuffix, ModuleToRecompile.NewModuleFilename );

	TArray< FModuleToRecompile > ModulesToRecompile;
	ModulesToRecompile.Add( MoveTemp(ModuleToRecompile) );
	ModulesRecentlyCompiledInTheEditor.Add(InModuleName);
	if (!RecompileModuleDLLs(ModulesToRecompile))
	{
		return false;
	}

	SlowTask.EnterProgressFrame();

	// Shutdown the module if it's already running
	if( bWasModuleLoaded )
	{
		Ar.Logf( TEXT( "Unloading module before compile." ) );
		ModuleManager.UnloadOrAbandonModuleWithCallback( InModuleName, Ar );
	}
	else
	{
		// Reset the module cache in case it's a new module that we probably didn't know about already.
		ModuleManager.ResetModulePathsCache();
	}

	// Reload the module if it was loaded before we recompiled
	if ((bWasModuleLoaded || !!(Flags & ERecompileModuleFlags::ForceCodeProject)) && !!(Flags & ERecompileModuleFlags::ReloadAfterRecompile))
	{
		FScopedHotReload Guard(Reload);
		Reload->SetSendReloadCompleteNotification(false);
		Ar.Logf( TEXT( "Reloading module %s after successful compile." ), *InModuleName.ToString() );
		if (!ModuleManager.LoadModuleWithCallback( InModuleName, Ar ))
		{
			return false;
		}

		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
	}

	if (!!(Flags & ERecompileModuleFlags::ForceCodeProject))
	{
		FCoreUObjectDelegates::ReloadCompleteDelegate.Broadcast(EReloadCompleteReason::HotReloadManual);
	}

	return true;
#else
	return false;
#endif // WITH_HOT_RELOAD
}

ECompilationResult::Type FHotReloadModule::DoHotReloadFromEditor(EHotReloadFlags Flags)
{
	// Get all game modules we want to compile
	const FModuleManager& ModuleManager = FModuleManager::Get();
	TArray<FString> GameModuleNames = UEHotReload_Private::GetGameModuleNames(ModuleManager);

	ECompilationResult::Type Result = ECompilationResult::Unsupported;

	UEHotReload_Private::FPackagesAndDependentNames PackagesAndDependentNames = UEHotReload_Private::SplitByPackagesAndDependentNames(GameModuleNames);

	// Analytics
	double Duration = 0.0;
	{
		FScopedDurationTimer Timer(Duration);

		Result = RebindPackagesInternal(PackagesAndDependentNames.Packages, PackagesAndDependentNames.DependentNames, Flags, *GLog);
	}

	RecordAnalyticsEvent(TEXT("Editor"), Result, Duration, PackagesAndDependentNames.Packages.Num(), PackagesAndDependentNames.DependentNames.Num());

	return Result;
}

ECompilationResult::Type FHotReloadModule::DoHotReloadInternal(const TMap<FName, FString>& ChangedModules, const TArray<UPackage*>& Packages, const TArray<FName>& InDependentModules, FOutputDevice& HotReloadAr)
{
#if WITH_HOT_RELOAD

	FModuleManager& ModuleManager = FModuleManager::Get();

	ModuleManager.ResetModulePathsCache();

	FFeedbackContext& ErrorsFC = UClass::GetDefaultPropertiesFeedbackContext();
	ErrorsFC.ClearWarningsAndErrors();

	// Rebind the hot reload DLL 
	FScopedHotReload Guard(Reload, Packages);
	Reload->SetSendReloadCompleteNotification(false);

	// we create a new CDO in the transient package...this needs to go away before we try again.
	CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS); 

	// Pretend we are loading.  This must happen after GC
	TGuardValue<bool> GuardIsInitialLoad(GIsInitialLoad, true);

	// Load the new modules up
	bool bReloadSucceeded = false;
	ECompilationResult::Type Result = ECompilationResult::Unsupported;
	for (UPackage* Package : Packages)
	{
		FString PackageName = Package->GetName();
		FName ShortPackageFName = *FPackageName::GetShortName(PackageName);

		if (!ChangedModules.Contains(ShortPackageFName))
		{
			continue;
		}

		// Abandon the old module.  We can't unload it because various data structures may be living
		// that have vtables pointing to code that would become invalidated.
		ModuleManager.AbandonModuleWithCallback(ShortPackageFName);

		// Load the newly-recompiled module up (it will actually have a different DLL file name at this point.)
		bReloadSucceeded = ModuleManager.LoadModule(ShortPackageFName) != nullptr;
		if (!bReloadSucceeded)
		{
			HotReloadAr.Logf(ELogVerbosity::Warning, TEXT("HotReload failed, reload failed %s."), *PackageName);
			Result = ECompilationResult::OtherCompilationError;
			break;
		}
	}

	// Load dependent modules.
	for (FName ModuleName : InDependentModules)
	{
		if (!ChangedModules.Contains(ModuleName))
		{
			continue;
		}

		ModuleManager.UnloadOrAbandonModuleWithCallback(ModuleName, HotReloadAr);
		const bool bLoaded = ModuleManager.LoadModuleWithCallback(ModuleName, HotReloadAr);
		if (!bLoaded)
		{
			HotReloadAr.Logf(ELogVerbosity::Warning, TEXT("Unable to reload module %s"), *ModuleName.GetPlainNameString());
		}
	}

	if (ErrorsFC.GetNumErrors() || ErrorsFC.GetNumWarnings())
	{
		TArray<FString> AllErrorsAndWarnings;
		ErrorsFC.GetErrorsAndWarningsAndEmpty(AllErrorsAndWarnings);

		FString AllInOne;
		for (const FString& ErrorOrWarning : AllErrorsAndWarnings)
		{
			AllInOne += ErrorOrWarning;
			AllInOne += TEXT("\n");
		}
		HotReloadAr.Logf(ELogVerbosity::Warning, TEXT("Some classes could not be reloaded:\n%s"), *AllInOne);
	}

	if (bReloadSucceeded)
	{
		Reload->Finalize();

		Result = ECompilationResult::Succeeded;
	}


	FCoreUObjectDelegates::ReloadCompleteDelegate.Broadcast(bIsHotReloadingFromEditor ? EReloadCompleteReason::HotReloadManual : EReloadCompleteReason::HotReloadAutomatic);

	HotReloadAr.Logf(ELogVerbosity::Display, TEXT("HotReload took %4.1fs."), FPlatformTime::Seconds() - HotReloadStartTime);

	bIsHotReloadingFromEditor = false;
	return Result;

#else

	bIsHotReloadingFromEditor = false;
	return ECompilationResult::Unsupported;

#endif
}

ECompilationResult::Type FHotReloadModule::RebindPackages(const TArray<UPackage*>& InPackages, EHotReloadFlags Flags, FOutputDevice &Ar)
{
	ECompilationResult::Type Result = ECompilationResult::Unknown;

	// Get game packages
	const FModuleManager& ModuleManager = FModuleManager::Get();
	TArray<FString> GameModuleNames = UEHotReload_Private::GetGameModuleNames(ModuleManager);
	UEHotReload_Private::FPackagesAndDependentNames PackagesAndDependentNames = UEHotReload_Private::SplitByPackagesAndDependentNames(GameModuleNames);

	// Get a set of source packages combined with game packages
	TSet<UPackage*> PackagesIncludingGame(InPackages);
	int32 NumInPackages = PackagesIncludingGame.Num();
	PackagesIncludingGame.Append(PackagesAndDependentNames.Packages);

	// Check if there was any overlap
	bool bInPackagesIncludeGame = PackagesIncludingGame.Num() < NumInPackages + PackagesAndDependentNames.Packages.Num();

	// If any of those modules were game modules, we'll compile those too
	TArray<UPackage*> Packages;
	TArray<FName>     Dependencies;
	if (bInPackagesIncludeGame)
	{
		Packages     = PackagesIncludingGame.Array();
		Dependencies = MoveTemp(PackagesAndDependentNames.DependentNames);
	}
	else
	{
		Packages = InPackages;
	}

	double Duration = 0.0;
	{
		FScopedDurationTimer RebindTimer(Duration);
		Result = RebindPackagesInternal(Packages, Dependencies, Flags, Ar);
	}
	RecordAnalyticsEvent(TEXT("Rebind"), Result, Duration, Packages.Num(), Dependencies.Num());

	return Result;
}

ECompilationResult::Type FHotReloadModule::RebindPackagesInternal(const TArray<UPackage*>& InPackages, const TArray<FName>& DependentModules, EHotReloadFlags Flags, FOutputDevice& Ar)
{
#if WITH_HOT_RELOAD
	if (InPackages.Num() == 0)
	{
		Ar.Logf(ELogVerbosity::Warning, TEXT("RebindPackages not possible (no packages specified)"));
		return ECompilationResult::Unsupported;
	}

	// Verify that we're going to be able to rebind the specified packages
	for (UPackage* Package : InPackages)
	{
		check(Package);

		if (Package->GetOuter())
		{
			Ar.Logf(ELogVerbosity::Warning, TEXT("Could not rebind package for %s, package is either not bound yet or is not a DLL."), *Package->GetName());
			return ECompilationResult::Unsupported;
		}
	}

	// We can only proceed if a compile isn't already in progress
	if (IsCurrentlyCompiling())
	{
		Ar.Logf(ELogVerbosity::Warning, TEXT("Could not rebind package because a module compile is already in progress."));
		return ECompilationResult::Unsupported;
	}

	FModuleManager::Get().ResetModulePathsCache();

	bIsHotReloadingFromEditor = true;

	HotReloadStartTime = FPlatformTime::Seconds();

	TArray< FName > ModuleNames;
	for (UPackage* Package : InPackages)
	{
		// Attempt to recompile this package's module
		FName ShortPackageName = FPackageName::GetShortFName(Package->GetFName());
		ModuleNames.Add(ShortPackageName);
	}

	// Add dependent modules.
	ModuleNames.Append(DependentModules);

	// Start compiling modules
	//
	// NOTE: This method of recompiling always using a rolling file name scheme, since we never want to unload before
	// we start recompiling, and we need the output DLL to be unlocked before we invoke the compiler

	ModuleCompilerStartedEvent.Broadcast(!(Flags & EHotReloadFlags::WaitForCompletion)); // we perform an async compile providing we're not waiting for completion

	FModuleManager& ModuleManager = FModuleManager::Get();

	TArray< FModuleToRecompile > ModulesToRecompile;
	for( FName CurModuleName : ModuleNames )
	{
		// Update our set of known modules, in case we don't already know about this module
		ModuleManager.AddModule( CurModuleName );

		// Find a unique file name for the module
		FModuleToRecompile ModuleToRecompile;
		ModuleToRecompile.ModuleName = CurModuleName;
		ModuleManager.MakeUniqueModuleFilename( CurModuleName, ModuleToRecompile.ModuleFileSuffix, ModuleToRecompile.NewModuleFilename );

		ModulesToRecompile.Add( ModuleToRecompile );
	}

	// Kick off compilation!
	const FString AdditionalArguments = MakeUBTArgumentsForModuleCompiling();
	bool bCompileStarted = StartCompilingModuleDLLs(
		ModulesToRecompile,
		[this, InPackages, DependentModules, &Ar](const TMap<FName, FString>& ChangedModules, bool bRecompileFinished, ECompilationResult::Type CompilationResult)
		{
			if (ECompilationResult::Failed(CompilationResult) && bRecompileFinished)
			{
				Ar.Logf(ELogVerbosity::Warning, TEXT("HotReload failed, recompile failed"));
				return;
			}

			DoHotReloadInternal(ChangedModules, InPackages, DependentModules, Ar);
		},
		Ar,
		AdditionalArguments,
		ERecompileModuleFlags::None
	);

	if (!bCompileStarted)
	{
		Ar.Logf(ELogVerbosity::Warning, TEXT("RebindPackages failed because the compiler could not be started."));
		bIsHotReloadingFromEditor = false;
		return ECompilationResult::OtherCompilationError;
	}

	// Go ahead and check for completion right away.  This is really just so that we can handle the case
	// where the user asked us to wait for the compile to finish before returning.
	if (!!(Flags & EHotReloadFlags::WaitForCompletion))
	{
		bool bCompileStillInProgress = false;
		bool bCompileSucceeded = false;
		FOutputDeviceNull NullOutput;
		CheckForFinishedModuleDLLCompile( Flags, bCompileStillInProgress, bCompileSucceeded, NullOutput );
		if( !bCompileStillInProgress && !bCompileSucceeded )
		{
			Ar.Logf(ELogVerbosity::Warning, TEXT("RebindPackages failed because compilation failed."));
			bIsHotReloadingFromEditor = false;
			return ECompilationResult::OtherCompilationError;
		}
	}

	if (!!(Flags & EHotReloadFlags::WaitForCompletion))
	{
		Ar.Logf(ELogVerbosity::Warning, TEXT("HotReload operation took %4.1fs."), float(FPlatformTime::Seconds() - HotReloadStartTime));
		bIsHotReloadingFromEditor = false;
	}
	else
	{
		Ar.Logf(ELogVerbosity::Warning, TEXT("Starting HotReload took %4.1fs."), float(FPlatformTime::Seconds() - HotReloadStartTime));
	}

	return ECompilationResult::Succeeded;
#else
	Ar.Logf(ELogVerbosity::Warning, TEXT("RebindPackages not possible (hot reload not supported)"));
	return ECompilationResult::Unsupported;
#endif
}

#if WITH_ENGINE
void FHotReloadModule::RegisterForReinstancing(UClass* OldClass, UClass* NewClass, EHotReloadedClassFlags Flags)
{
	
	// For compatibility, monitor this broadcast.  If we don't have an active reload then we create one assuming
	// the broadcaster wanted to reinstance (i.e. python wrapper).  
	IReload* TempReload = GetActiveReloadInterface();
	if (TempReload == nullptr)
	{
		Reload.Reset(new FReload(EActiveReloadType::Reinstancing, TEXT(""), *GLog));
#if WITH_RELOAD
		BeginReload(Reload->GetType(), *Reload);
#endif
		TempReload = Reload.Get();
	}

	// Only invoke the notification if we own the reload object and it is the temporary reinstancing object.
	if (TempReload == Reload.Get() && TempReload->GetType() == EActiveReloadType::Reinstancing)
	{
		TempReload->NotifyChange(EnumHasAnyFlags(Flags, EHotReloadedClassFlags::Changed) ? NewClass : OldClass, OldClass);
	}
}

void FHotReloadModule::ReinstanceClasses()
{
	// Only invoke the notification if we own the reload object and it is the temporary reinstancing object.
	IReload* TempReload = GetActiveReloadInterface();
	if (TempReload == Reload.Get() && TempReload->GetType() == EActiveReloadType::Reinstancing)
	{
		TempReload->Reinstance();
		Reload.Reset();
	}
}
#endif

void FHotReloadModule::OnHotReloadBinariesChanged(const TArray<FFileChangeData>& FileChanges)
{
	if (bIsHotReloadingFromEditor)
	{
		// DO NOTHING, this case is handled by RebindPackages
		return;
	}

	const FModuleManager& ModuleManager = FModuleManager::Get();
	TMap<FName, FString> GameModuleFilenames = UEHotReload_Private::GetGameModuleFilenames(ModuleManager);

	if (GameModuleFilenames.Num() == 0)
	{
		return;
	}

	// Check if any of the game DLLs has been added
	for (const FFileChangeData& Change : FileChanges)
	{
		// Ignore changes that aren't introducing a new file.
		//
		// On the Mac the Add event is for a temporary linker(?) file that gets immediately renamed
		// to a dylib. In the future we may want to support modified event for all platforms anyway once
		// shadow copying works with hot-reload.
#if PLATFORM_MAC
		if (Change.Action != FFileChangeData::FCA_Modified)
#else
		if (Change.Action != FFileChangeData::FCA_Added)
#endif
		{
			continue;
		}

		// Ignore files that aren't of module type
		FString Filename = FPaths::GetCleanFilename(Change.Filename);
		if (!Filename.EndsWith(FPlatformProcess::GetModuleExtension()))
		{
			continue;
		}

		for (const TPair<FName, FString>& NameFilename : GameModuleFilenames)
		{
			// Handle module files which have already been hot-reloaded.
			FString BaseName = FPaths::GetBaseFilename(NameFilename.Value);
			StripModuleSuffixFromFilename(BaseName, NameFilename.Key.ToString());

			// Hot reload always adds a numbered suffix preceded by a hyphen, but otherwise the module name must match exactly!
			if (!Filename.StartsWith(BaseName + TEXT("-")))
			{
				continue;
			}

			if (ModulesRecentlyCompiledInTheEditor.Contains(NameFilename.Key))
			{
				continue;
			}

			// Add to queue. We do not hot-reload here as there may potentially be other modules being compiled.
			DetectedNewModules.Emplace(NameFilename.Key, Change.Filename);
			UE_LOGF(LogHotReload, Log, "New module detected: %ls", *Filename);
		}
	}
}

void FHotReloadModule::StripModuleSuffixFromFilename(FString& InOutModuleFilename, const FString& ModuleName)
{
	// First hyphen is where the UEEdtior prefix ends
	int32 FirstHyphenIndex = INDEX_NONE;
	if (InOutModuleFilename.FindChar('-', FirstHyphenIndex))
	{
		// Second hyphen means we already have a hot-reloaded module or other than Development config module
		int32 SecondHyphenIndex = FirstHyphenIndex;
		do
		{
			SecondHyphenIndex = InOutModuleFilename.Find(TEXT("-"), ESearchCase::CaseSensitive, ESearchDir::FromStart, SecondHyphenIndex + 1);
			if (SecondHyphenIndex != INDEX_NONE)
			{
				// Make sure that the section between hyphens is the expected module name. This guards against cases where module name has a hyphen inside.
				FString HotReloadedModuleName = InOutModuleFilename.Mid(FirstHyphenIndex + 1, SecondHyphenIndex - FirstHyphenIndex - 1);
				if (HotReloadedModuleName == ModuleName)
				{
					InOutModuleFilename.MidInline(0, SecondHyphenIndex, EAllowShrinking::No);
					SecondHyphenIndex = INDEX_NONE;
				}
			}
		} while (SecondHyphenIndex != INDEX_NONE);
	}
}

void FHotReloadModule::RefreshHotReloadWatcher()
{
	FDirectoryWatcherModule& DirectoryWatcherModule = FModuleManager::Get().LoadModuleChecked<FDirectoryWatcherModule>(TEXT("DirectoryWatcher"));
	IDirectoryWatcher* DirectoryWatcher = DirectoryWatcherModule.Get();
	if (DirectoryWatcher)
	{
		// Watch the game directory
		AddHotReloadDirectory(DirectoryWatcher, FPaths::ProjectDir());

		// Also watch all the game plugin directories
		for(const TSharedRef<IPlugin>& Plugin : IPluginManager::Get().GetEnabledPlugins())
		{
			if (Plugin->GetLoadedFrom() == EPluginLoadedFrom::Project && Plugin->GetDescriptor().Modules.Num() > 0)
			{
				AddHotReloadDirectory(DirectoryWatcher, Plugin->GetBaseDir());
			}
		}
	}
}

void FHotReloadModule::AddHotReloadDirectory(IDirectoryWatcher* DirectoryWatcher, const FString& BaseDir)
{
	FString BinariesPath = FPaths::ConvertRelativePathToFull(BaseDir / TEXT("Binaries") / FPlatformProcess::GetBinariesSubdirectory());
	if (FPaths::DirectoryExists(BinariesPath) && !BinariesFolderChangedDelegateHandles.Contains(BinariesPath))
	{
		IDirectoryWatcher::FDirectoryChanged BinariesFolderChangedDelegate = IDirectoryWatcher::FDirectoryChanged::CreateRaw(this, &FHotReloadModule::OnHotReloadBinariesChanged);

		FDelegateHandle Handle;
		if (DirectoryWatcher->RegisterDirectoryChangedCallback_Handle(BinariesPath, BinariesFolderChangedDelegate, Handle))
		{
			BinariesFolderChangedDelegateHandles.Add(BinariesPath, Handle);
		}
	}
}

void FHotReloadModule::ShutdownHotReloadWatcher()
{
	FDirectoryWatcherModule* DirectoryWatcherModule = FModuleManager::GetModulePtr<FDirectoryWatcherModule>(TEXT("DirectoryWatcher"));
	if( DirectoryWatcherModule != nullptr )
	{
		IDirectoryWatcher* DirectoryWatcher = DirectoryWatcherModule->Get();
		if (DirectoryWatcher)
		{
			for (const TPair<FString, FDelegateHandle>& Pair : BinariesFolderChangedDelegateHandles)
			{
				DirectoryWatcher->UnregisterDirectoryChangedCallback_Handle(Pair.Key, Pair.Value);
			}
		}
	}
}

bool FHotReloadModule::Tick(float DeltaTime)
{
    QUICK_SCOPE_CYCLE_COUNTER(STAT_FHotReloadModule_Tick);

	// We never want to block on a pending compile when checking compilation status during Tick().  We're
	// just checking so that we can fire callbacks if and when compilation has finished.
	// Ignored output variables
	bool bCompileStillInProgress = false;
	bool bCompileSucceeded = false;
	FOutputDeviceNull NullOutput;
	CheckForFinishedModuleDLLCompile( EHotReloadFlags::None, bCompileStillInProgress, bCompileSucceeded, NullOutput );

	if (DetectedNewModules.Num() == 0)
	{
		return true;
	}

	// Early out if live coding is enabled
#if WITH_LIVE_CODING
	ILiveCodingModule* LiveCoding = FModuleManager::GetModulePtr<ILiveCodingModule>(LIVE_CODING_MODULE_NAME);
	if (LiveCoding != nullptr && LiveCoding->IsEnabledForSession())
	{
        return false;
	}
#endif

#if WITH_EDITOR
	if (GEditor)
	{
		// Don't try to do an IDE reload yet if we're PIE - wait until we leave
		if (GEditor->IsPlaySessionInProgress())
		{
			return true;
		}

		// Don't allow hot reloading if we're running networked PIE instances
		// The reason, is it's fairly complicated to handle the re-wiring that needs to happen when we re-instance objects like player controllers, possessed pawns, etc...
		const TIndirectArray<FWorldContext>& WorldContextList = GEditor->GetWorldContexts();

		for (const FWorldContext& WorldContext : WorldContextList)
		{
			if (WorldContext.World() && WorldContext.World()->WorldType == EWorldType::PIE && WorldContext.World()->NetDriver)
			{
				return true;		// Don't allow automatic hot reloading if we're running PIE instances
			}
		}
	}
#endif // WITH_EDITOR

	// We have new modules in the queue, but make sure UBT has finished compiling all of them
	if (!FDesktopPlatformModule::Get()->IsUnrealBuildToolRunning())
	{
		IFileManager& FileManager = IFileManager::Get();

		// Remove any modules whose files have disappeared - this can happen if a compile event has
		// failed and deleted a DLL that was there previously.
		for (auto It = DetectedNewModules.CreateIterator(); It; ++It)
		{
			if (!FileManager.FileExists(*It->Value))
			{
				It.RemoveCurrent();
			}
		}
		DoHotReloadFromIDE(DetectedNewModules);
		DetectedNewModules.Empty();
	}
	else
	{
		UE_LOGF(LogHotReload, Verbose, "Detected %d reloaded modules but UnrealBuildTool is still running", DetectedNewModules.Num());
	}

	return true;
}

void FHotReloadModule::DoHotReloadFromIDE(const TMap<FName, FString>& NewModules)
{
	const FModuleManager& ModuleManager = FModuleManager::Get();

	int32 NumPackagesToRebind = 0;
	int32 NumDependentModules = 0;

	ECompilationResult::Type Result = ECompilationResult::Unsupported;

	double Duration = 0.0;

	TArray<FString> GameModuleNames = UEHotReload_Private::GetGameModuleNames(ModuleManager);
	if (GameModuleNames.Num() > 0)
	{
		FScopedDurationTimer Timer(Duration);

		if (NewModules.Num() == 0)
		{
			return;
		}

		UE_LOGF(LogHotReload, Log, "Starting Hot-Reload from IDE");

		HotReloadStartTime = FPlatformTime::Seconds();

		FScopedSlowTask SlowTask(100.f, LOCTEXT("CompilingGameCode", "Compiling Game Code"));
		SlowTask.MakeDialog();

		// Update compile data before we start compiling
		for (const TPair<FName, FString>& NewModule : NewModules)
		{
			// Move on 10% / num items
			SlowTask.EnterProgressFrame(10.f / NewModules.Num());

			FName ModuleName = NewModule.Key;

			UpdateModuleCompileData(ModuleName);
			OnModuleCompileSucceeded(ModuleName, NewModule.Value);
		}

		SlowTask.EnterProgressFrame(10);
		UEHotReload_Private::FPackagesAndDependentNames PackagesAndDependentNames = UEHotReload_Private::SplitByPackagesAndDependentNames(GameModuleNames);
		SlowTask.EnterProgressFrame(80);

		NumPackagesToRebind = PackagesAndDependentNames.Packages.Num();
		NumDependentModules = PackagesAndDependentNames.DependentNames.Num();
		Result = DoHotReloadInternal(NewModules, PackagesAndDependentNames.Packages, PackagesAndDependentNames.DependentNames, *GLog);
	}

	RecordAnalyticsEvent(TEXT("IDE"), Result, Duration, NumPackagesToRebind, NumDependentModules);
}

void FHotReloadModule::RecordAnalyticsEvent(const TCHAR* ReloadFrom, ECompilationResult::Type Result, double Duration, int32 PackageCount, int32 DependentModulesCount)
{
#if WITH_ENGINE
	if (FEngineAnalytics::IsAvailable())
	{
		TArray< FAnalyticsEventAttribute > ReloadAttribs;
		ReloadAttribs.Add(FAnalyticsEventAttribute(TEXT("ReloadFrom"), ReloadFrom));
		ReloadAttribs.Add(FAnalyticsEventAttribute(TEXT("Result"), ECompilationResult::ToString(Result)));
		ReloadAttribs.Add(FAnalyticsEventAttribute(TEXT("Duration"), FString::Printf(TEXT("%.4lf"), Duration)));
		ReloadAttribs.Add(FAnalyticsEventAttribute(TEXT("Packages"), FString::Printf(TEXT("%d"), PackageCount)));
		ReloadAttribs.Add(FAnalyticsEventAttribute(TEXT("DependentModules"), FString::Printf(TEXT("%d"), DependentModulesCount)));
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.HotReload"), ReloadAttribs);
	}
#endif
}

void FHotReloadModule::OnModuleCompileSucceeded(FName ModuleName, const FString& ModuleFilename)
{
#if !IS_MONOLITHIC
		// If the compile succeeded, update the module info entry with the new file name for this module
	FModuleManager::Get().SetModuleFilename(ModuleName, ModuleFilename);
#endif

#if WITH_HOT_RELOAD
	// UpdateModuleCompileData() should have been run before compiling so the
	// data in ModuleInfo should be correct for the pre-compile dll file.
	FModuleCompilationData& CompileData = ModuleCompileData.FindChecked(ModuleName).Get();

	FDateTime FileTimeStamp;
	bool bGotFileTimeStamp = GetModuleFileTimeStamp(ModuleName, FileTimeStamp);

	CompileData.bHasFileTimeStamp = bGotFileTimeStamp;
	CompileData.FileTimeStamp = FileTimeStamp;

	if (CompileData.bHasFileTimeStamp)
	{
		CompileData.CompileMethod = EModuleCompileMethod::Runtime;
	}
	else
	{
		CompileData.CompileMethod = EModuleCompileMethod::Unknown;
	}
	WriteModuleCompilationInfoToConfig(ModuleName, CompileData);
#endif
}

FString FHotReloadModule::MakeUBTArgumentsForModuleCompiling()
{
	FString AdditionalArguments;
	if ( FPaths::IsProjectFilePathSet() )
	{
		// We have to pass FULL paths to UBT
		FString FullProjectPath = FPaths::ConvertRelativePathToFull(FPaths::GetProjectFilePath());
		AdditionalArguments += FString::Printf(TEXT("\"%s\" "), *FullProjectPath);
	}
	
#if PLATFORM_MAC_ARM64
	AdditionalArguments += "-architecture=arm64";
#endif

	return AdditionalArguments;
}

#if WITH_HOT_RELOAD
bool FHotReloadModule::StartCompilingModuleDLLs(const TArray< FModuleToRecompile >& ModuleNames,
	FRecompileModulesCallback&& InRecompileModulesCallback, FOutputDevice& Ar,
	const FString& InAdditionalCmdLineArgs, ERecompileModuleFlags Flags)
{
	// Keep track of what we're compiling
	ModulesBeingCompiled.Empty(ModuleNames.Num());
	Algo::Transform(ModuleNames, ModulesBeingCompiled, &FModuleToRecompile::ModuleName);
	ModulesThatWereBeingRecompiled = ModuleNames;

	const TCHAR* BuildPlatformName = FPlatformMisc::GetUBTPlatform();
	const TCHAR* BuildConfigurationName = FModuleManager::GetUBTConfiguration();
	const TCHAR* BuildTargetName = FPlatformMisc::GetUBTTargetName();

	RecompileModulesCallback = MoveTemp(InRecompileModulesCallback);

	// Pass a module file suffix to UBT if we have one
	FString ModuleArg;
	if (ModuleNames.Num())
	{
		Ar.Logf(TEXT("Candidate modules for hot reload:"));
		for( const FModuleToRecompile& Module : ModuleNames )
		{
			FString ModuleNameStr = Module.ModuleName.ToString();

			if( !Module.ModuleFileSuffix.IsEmpty() )
			{
				ModuleArg += FString::Printf( TEXT( " -ModuleWithSuffix=%s,%s" ), *ModuleNameStr, *Module.ModuleFileSuffix );
			}
			else
			{
				ModuleArg += FString::Printf( TEXT( " -Module=%s" ), *ModuleNameStr );
			}
			Ar.Logf( TEXT( "  %s" ), *ModuleNameStr );

			// prepare the compile info in the FModuleInfo so that it can be compared after compiling
			UpdateModuleCompileData(Module.ModuleName);
		}
	}

	FString ExtraArg;

	if (FPaths::IsProjectFilePathSet())
	{
		ExtraArg += FString::Printf(TEXT("-Project=\"%s\" "), *FPaths::ConvertRelativePathToFull(FPaths::GetProjectFilePath()));
	}

	if (!!(Flags & ERecompileModuleFlags::FailIfGeneratedCodeChanges))
	{
		// Additional argument to let UHT know that we can only compile the module if the generated code didn't change
		ExtraArg += TEXT( "-FailIfGeneratedCodeChanges " );
	}

	FString CmdLineParams = FString::Printf( TEXT( "%s %s %s %s %s%s -IgnoreJunk" ), 
		*ModuleArg, 
		BuildTargetName, BuildPlatformName, BuildConfigurationName,
		*ExtraArg, *InAdditionalCmdLineArgs );

	const bool bInvocationSuccessful = InvokeUnrealBuildToolForCompile(CmdLineParams, Ar);
	if ( !bInvocationSuccessful )
	{
		// No longer compiling modules
		ModulesBeingCompiled.Empty();

		ModuleCompilerFinishedEvent.Broadcast(FString(), ECompilationResult::OtherCompilationError, false);

		// Fire task completion delegate 
		
		if (RecompileModulesCallback)
		{
			RecompileModulesCallback( TMap<FName, FString>(), false, ECompilationResult::OtherCompilationError );
			RecompileModulesCallback = nullptr;
		}
	}

	return bInvocationSuccessful;
}
#endif

bool FHotReloadModule::InvokeUnrealBuildToolForCompile(const FString& InCmdLineParams, FOutputDevice &Ar)
{
#if WITH_HOT_RELOAD

	// Make sure we're not already compiling something!
	check(!IsCurrentlyCompiling());

	// Setup output redirection pipes, so that we can harvest compiler output and display it ourselves
	void* PipeRead = NULL;
	void* PipeWrite = NULL;

	verify(FPlatformProcess::CreatePipe(PipeRead, PipeWrite));
	ModuleCompileReadPipeText = TEXT("");

	FProcHandle ProcHandle = FDesktopPlatformModule::Get()->InvokeUnrealBuildToolAsync(InCmdLineParams, Ar, PipeRead, PipeWrite);

	// We no longer need the Write pipe so close it.
	// We DO need the Read pipe however...
	FPlatformProcess::ClosePipe(0, PipeWrite);

	if (!ProcHandle.IsValid())
	{
		// We're done with the process handle now
		ModuleCompileProcessHandle.Reset();
		ModuleCompileReadPipe = NULL;
	}
	else
	{
		ModuleCompileProcessHandle = ProcHandle;
		ModuleCompileReadPipe = PipeRead;
	}

	return ProcHandle.IsValid();
#else
	return false;
#endif // WITH_HOT_RELOAD
}

void FHotReloadModule::CheckForFinishedModuleDLLCompile(EHotReloadFlags Flags, bool& bCompileStillInProgress, bool& bCompileSucceeded, FOutputDevice& Ar, bool bFireEvents)
{
#if WITH_HOT_RELOAD
	bCompileStillInProgress = false;
	ECompilationResult::Type CompilationResult = ECompilationResult::OtherCompilationError;

	// Is there a compilation in progress?
	if( !IsCurrentlyCompiling() )
	{
		Ar.Logf(TEXT("Error: CheckForFinishedModuleDLLCompile: There is no compilation in progress right now"));
		return;
	}

	bCompileStillInProgress = true;

	FText StatusUpdate;
	if ( ModulesBeingCompiled.Num() > 0 )
	{
		FFormatNamedArguments Args;
		Args.Add( TEXT("CodeModuleName"), FText::FromName( ModulesBeingCompiled[0] ) );
		StatusUpdate = FText::Format( NSLOCTEXT("FModuleManager", "CompileSpecificModuleStatusMessage", "{CodeModuleName}: Compiling modules..."), Args );
	}
	else
	{
		StatusUpdate = NSLOCTEXT("FModuleManager", "CompileStatusMessage", "Compiling modules...");
	}

	FScopedSlowTask SlowTask(0, StatusUpdate, GIsSlowTask);
	SlowTask.MakeDialog();

	// Check to see if the compile has finished yet
	int32 ReturnCode = -1;
	while (bCompileStillInProgress)
	{
		// Store the return code in a temp variable for now because it still gets overwritten
		// when the process is running.
		int32 ProcReturnCode = -1;
		if( FPlatformProcess::GetProcReturnCode( ModuleCompileProcessHandle, &ProcReturnCode ) )
		{
			ReturnCode = ProcReturnCode;
			bCompileStillInProgress = false;
		}
		
		if (bRequestCancelCompilation)
		{
			FPlatformProcess::TerminateProc(ModuleCompileProcessHandle);
			bCompileStillInProgress = bRequestCancelCompilation = false;
		}

		if( bCompileStillInProgress )
		{
			ModuleCompileReadPipeText += FPlatformProcess::ReadPipe(ModuleCompileReadPipe);

			if (!(Flags & EHotReloadFlags::WaitForCompletion))
			{
				// We haven't finished compiling, but we were asked to return immediately
				break;
			}

			SlowTask.EnterProgressFrame(0.0f);

			// Give up a small timeslice if we haven't finished recompiling yet
			FPlatformProcess::Sleep( 0.01f );
		}
	}
	
	bRequestCancelCompilation = false;

	if( bCompileStillInProgress )
	{
		Ar.Logf(TEXT("Error: CheckForFinishedModuleDLLCompile: Compilation is still in progress"));
		return;
	}

	// Compilation finished, now we need to grab all of the text from the output pipe
	ModuleCompileReadPipeText += FPlatformProcess::ReadPipe(ModuleCompileReadPipe);

	// This includes 'canceled' (-1) and 'up-to-date' (-2)
	CompilationResult = (ECompilationResult::Type)ReturnCode;

	// If compilation succeeded for all modules, go back to the modules and update their module file names
	// in case we recompiled the modules to a new unique file name.  This is needed so that when the module
	// is reloaded after the recompile, we load the new DLL file name, not the old one.
	// Note that we don't want to do anything in case the build was canceled or source code has not changed.
	TMap<FName, FString> ChangedModules;
	if(CompilationResult == ECompilationResult::Succeeded)
	{
		ChangedModules.Reserve(ModulesThatWereBeingRecompiled.Num());
		for( FModuleToRecompile& CurModule : ModulesThatWereBeingRecompiled )
		{
			bool bModuleChanged = !CurModule.NewModuleFilename.IsEmpty();

			// Were we asked to assign a new file name for this module?
			if (!bModuleChanged)
			{
				FModuleManager& ModuleManager = FModuleManager::Get();

				// This is a new module, so reset the cache and find the name of it.
				ModuleManager.ResetModulePathsCache();
				ModuleManager.RefreshModuleFilenameFromManifest(CurModule.ModuleName);
				CurModule.NewModuleFilename = ModuleManager.GetModuleFilename(CurModule.ModuleName);
			}

			if (IFileManager::Get().FileSize(*CurModule.NewModuleFilename) <= 0)
			{
				continue;
			}

			// If the file doesn't exist, then assume it doesn't needs rebinding because it wasn't recompiled
			FDateTime FileTimeStamp = IFileManager::Get().GetTimeStamp(*CurModule.NewModuleFilename);
			if (FileTimeStamp == FDateTime::MinValue())
			{
				continue;
			}

			// If the file is the same as what we remembered it was then assume it doesn't needs rebinding because it wasn't recompiled
			TSharedRef<FModuleCompilationData>* CompileDataPtr = ModuleCompileData.Find(CurModule.ModuleName);
			if (CompileDataPtr && (*CompileDataPtr)->FileTimeStamp == FileTimeStamp)
			{
				continue;
			}

			// If the compile succeeded, update the module info entry with the new file name for this module
			OnModuleCompileSucceeded(CurModule.ModuleName, CurModule.NewModuleFilename);

			if (bModuleChanged)
			{
				// Move modules
				ChangedModules.Emplace(CurModule.ModuleName, MoveTemp(CurModule.NewModuleFilename));
			}
		}
	}
	ModulesThatWereBeingRecompiled.Empty();

	// We're done with the process handle now
	FPlatformProcess::CloseProc(ModuleCompileProcessHandle);
	ModuleCompileProcessHandle.Reset();

	FPlatformProcess::ClosePipe(ModuleCompileReadPipe, 0);

	Ar.Log(*ModuleCompileReadPipeText);
	const FString FinalOutput = ModuleCompileReadPipeText;
	ModuleCompileReadPipe = NULL;
	ModuleCompileReadPipeText = TEXT("");

	// No longer compiling modules
	ModulesBeingCompiled.Empty();

	bCompileSucceeded = !ECompilationResult::Failed(CompilationResult);

	if ( bFireEvents )
	{
		const bool bShowLogOnSuccess = false;
		ModuleCompilerFinishedEvent.Broadcast(FinalOutput, CompilationResult, !bCompileSucceeded || bShowLogOnSuccess);

		// Fire task completion delegate 
		if (RecompileModulesCallback)
		{
			RecompileModulesCallback( ChangedModules, true, CompilationResult );
			RecompileModulesCallback = nullptr;
		}
	}
#endif // WITH_HOT_RELOAD
}

void FHotReloadModule::UpdateModuleCompileData(FName ModuleName)
{
	// Find or create a compile data object for this module
	TSharedRef<FModuleCompilationData>* CompileDataPtr = ModuleCompileData.Find(ModuleName);
	if(CompileDataPtr == nullptr)
	{
		CompileDataPtr = &ModuleCompileData.Add(ModuleName, TSharedRef<FModuleCompilationData>(new FModuleCompilationData()));
	}

	// reset the compile data before updating it
	FModuleCompilationData& CompileData = CompileDataPtr->Get();
	CompileData.bHasFileTimeStamp = false;
	CompileData.FileTimeStamp = FDateTime(0);
	CompileData.CompileMethod = EModuleCompileMethod::Unknown;

#if WITH_HOT_RELOAD
	ReadModuleCompilationInfoFromConfig(ModuleName, CompileData);

	FDateTime FileTimeStamp;
	bool bGotFileTimeStamp = GetModuleFileTimeStamp(ModuleName, FileTimeStamp);

	if (!bGotFileTimeStamp)
	{
		// File missing? Reset the cached timestamp and method to defaults and save them.
		CompileData.bHasFileTimeStamp = false;
		CompileData.FileTimeStamp = FDateTime(0);
		CompileData.CompileMethod = EModuleCompileMethod::Unknown;
		WriteModuleCompilationInfoToConfig(ModuleName, CompileData);
	}
	else
	{
		if (CompileData.bHasFileTimeStamp)
		{
			if (FileTimeStamp > CompileData.FileTimeStamp + HotReloadDefs::TimeStampEpsilon)
			{
				// The file is newer than the cached timestamp
				// The file must have been compiled externally
				CompileData.FileTimeStamp = FileTimeStamp;
				CompileData.CompileMethod = EModuleCompileMethod::External;
				WriteModuleCompilationInfoToConfig(ModuleName, CompileData);
			}
		}
		else
		{
			// The cached timestamp and method are default value so this file has no history yet
			// We can only set its timestamp and save
			CompileData.bHasFileTimeStamp = true;
			CompileData.FileTimeStamp = FileTimeStamp;
			WriteModuleCompilationInfoToConfig(ModuleName, CompileData);
		}
	}
#endif
}

void FHotReloadModule::ReadModuleCompilationInfoFromConfig(FName ModuleName, FModuleCompilationData& CompileData)
{
	FString DateTimeString;
	if (GConfig->GetString(HotReloadDefs::CompilationInfoConfigSection, *FString::Printf(TEXT("%s.TimeStamp"), *ModuleName.ToString()), DateTimeString, GEditorPerProjectIni))
	{
		FDateTime TimeStamp;
		if (!DateTimeString.IsEmpty() && FDateTime::Parse(DateTimeString, TimeStamp))
		{
			CompileData.bHasFileTimeStamp = true;
			CompileData.FileTimeStamp = TimeStamp;

			FString CompileMethodString;
			if (GConfig->GetString(HotReloadDefs::CompilationInfoConfigSection, *FString::Printf(TEXT("%s.LastCompileMethod"), *ModuleName.ToString()), CompileMethodString, GEditorPerProjectIni))
			{
				if (FCString::Stricmp(*CompileMethodString, HotReloadDefs::CompileMethodRuntime) == 0)
				{
					CompileData.CompileMethod = EModuleCompileMethod::Runtime;
				}
				else if (FCString::Stricmp(*CompileMethodString, HotReloadDefs::CompileMethodExternal) == 0)
				{
					CompileData.CompileMethod = EModuleCompileMethod::External;
				}
			}
		}
	}
}

void FHotReloadModule::WriteModuleCompilationInfoToConfig(FName ModuleName, const FModuleCompilationData& CompileData)
{
	FString DateTimeString;
	if (CompileData.bHasFileTimeStamp)
	{
		DateTimeString = CompileData.FileTimeStamp.ToString();
	}

	GConfig->SetString(HotReloadDefs::CompilationInfoConfigSection, *FString::Printf(TEXT("%s.TimeStamp"), *ModuleName.ToString()), *DateTimeString, GEditorPerProjectIni);

	const TCHAR* CompileMethodString = HotReloadDefs::CompileMethodUnknown;
	if (CompileData.CompileMethod == EModuleCompileMethod::Runtime)
	{
		CompileMethodString = HotReloadDefs::CompileMethodRuntime;
	}
	else if (CompileData.CompileMethod == EModuleCompileMethod::External)
	{
		CompileMethodString = HotReloadDefs::CompileMethodExternal;
	}

	GConfig->SetString(HotReloadDefs::CompilationInfoConfigSection, *FString::Printf(TEXT("%s.LastCompileMethod"), *ModuleName.ToString()), CompileMethodString, GEditorPerProjectIni);
}

bool FHotReloadModule::GetModuleFileTimeStamp(FName ModuleName, FDateTime& OutFileTimeStamp) const
{
#if !IS_MONOLITHIC
	FString Filename = FModuleManager::Get().GetModuleFilename(ModuleName);
	if (IFileManager::Get().FileSize(*Filename) > 0)
	{
		OutFileTimeStamp = FDateTime(IFileManager::Get().GetTimeStamp(*Filename));
		return true;
	}
#endif
	return false;
}

bool FHotReloadModule::IsAnyGameModuleLoaded()
{
	if (bIsAnyGameModuleLoaded == EThreeStateBool::Unknown)
	{
		bool bGameModuleFound = false;
		// Ask the module manager for a list of currently-loaded gameplay modules
		TArray< FModuleStatus > ModuleStatuses;
		FModuleManager::Get().QueryModules(ModuleStatuses);

		for (const FModuleStatus& ModuleStatus : ModuleStatuses)
		{
			// We only care about game modules that are currently loaded
			if (ModuleStatus.bIsLoaded && ModuleStatus.bIsGameModule)
			{
				// There is at least one loaded game module.
				bGameModuleFound = true;
				break;
			}
		}
		bIsAnyGameModuleLoaded = EThreeStateBool::FromBool(bGameModuleFound);
	}
	return EThreeStateBool::ToBool(bIsAnyGameModuleLoaded);
}

bool FHotReloadModule::ContainsOnlyGameModules(const TArray<FModuleToRecompile>& ModulesToCompile) const
{
	FString AbsoluteProjectDir = FPaths::ConvertRelativePathToFull(FPaths::ProjectDir());
	for (const FModuleToRecompile& ModuleToCompile : ModulesToCompile)
	{
		FString FullModulePath = FPaths::ConvertRelativePathToFull(ModuleToCompile.NewModuleFilename);
		if (!FullModulePath.StartsWith(AbsoluteProjectDir))
		{
			return false;
		}
	}
	return true;
}

void FHotReloadModule::ModulesChangedCallback(FName ModuleName, EModuleChangeReason ReasonForChange)
{
	// Force update game modules state on the next call to IsAnyGameModuleLoaded
	bIsAnyGameModuleLoaded = EThreeStateBool::Unknown;
	
	// If the hot reload directory watcher hasn't been initialized yet (because the binaries directory did not exist) try to initialize it now
	if (!bDirectoryWatcherInitialized)
	{
		RefreshHotReloadWatcher();
		bDirectoryWatcherInitialized = true;
	}
}

void FHotReloadModule::PluginMountedCallback(IPlugin& Plugin)
{
	FDirectoryWatcherModule& DirectoryWatcherModule = FModuleManager::Get().LoadModuleChecked<FDirectoryWatcherModule>(TEXT("DirectoryWatcher"));

	IDirectoryWatcher* DirectoryWatcher = DirectoryWatcherModule.Get();
	if (DirectoryWatcher)
	{
		if (Plugin.GetLoadedFrom() == EPluginLoadedFrom::Project && Plugin.GetDescriptor().Modules.Num() > 0)
		{
			AddHotReloadDirectory(DirectoryWatcher, Plugin.GetBaseDir());
		}
	}
}

#undef LOCTEXT_NAMESPACE
